// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

public class Instrumentation : ModuleRules
{
	public Instrumentation(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresImplementModule = false;
		PrivateIncludePathModuleNames.Add("Core");

		PrivateDefinitions.Add("SUPPRESS_PER_MODULE_INLINE_FILE"); // This module does not use core's standard operator new/delete overloads
		
		bDisableAutoRTFMInstrumentation = true;

		// Instrumentation symbols (__Thunk__*) must be exported even with merged modules,
		// since they are dllimport'd by PerModuleInline.inl in every other module/DLL.
		ModuleSymbolVisibility = SymbolVisibility.VisibileForDll;
	}
}
