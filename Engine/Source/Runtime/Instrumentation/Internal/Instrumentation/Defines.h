// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define INSTRUMENTATION_FUNCTION_BASE_ATTRIBUTES __declspec(safebuffers) __attribute__((disable_sanitizer_instrumentation))

#if defined(__AUTORTFM) && __AUTORTFM
	#include "AutoRTFM/Constants.h"
	// autortfm_mode_internal are prevented from being inlined, so we only use that attribute on entry points
	#define INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES INSTRUMENTATION_FUNCTION_BASE_ATTRIBUTES __attribute__((autortfm(autortfm_mode_internal)))
	// use autortfm disable once we're inside race detection code to allow inlining
	#define INSTRUMENTATION_FUNCTION_ATTRIBUTES   INSTRUMENTATION_FUNCTION_BASE_ATTRIBUTES __attribute__((autortfm(autortfm_mode_disable)))
#else
	#define INSTRUMENTATION_FUNCTION_ATTRIBUTES   INSTRUMENTATION_FUNCTION_BASE_ATTRIBUTES
	#define INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES INSTRUMENTATION_FUNCTION_BASE_ATTRIBUTES
#endif

// Reserves 12 NOPs before the function and 2 NOPs at function entry so that we can patch atomically
#define INSTRUMENTATION_HOTPATCH_TOTAL_NOPS  14
#define INSTRUMENTATION_HOTPATCH_PREFIX_NOPS 12

#define INSTRUMENTATION_FUNCTION_HOTPATCHABLE __attribute__((patchable_function_entry(INSTRUMENTATION_HOTPATCH_TOTAL_NOPS, INSTRUMENTATION_HOTPATCH_PREFIX_NOPS)))