// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "Instrumentation/Types.h"
#include "Instrumentation/Defines.h"

extern "C"
{
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__AnnotateHappensBefore(const char* f, int l, void* addr);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__AnnotateHappensAfter(const char* f, int l, void* addr);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_FuncEntry(void* ReturnAddress);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_FuncExit();
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_Store(uint64 Address, uint32 Size);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_Load(uint64 Address, uint32 Size);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_VPtr_Store(void** Address, void* Value);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_VPtr_Load(void** Address);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_StoreRange(uint64 Address, uint32 Size);
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_LoadRange(uint64 Address, uint32 Size);

	#define INSTRUMENT_LOAD_FUNC(Type) \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API Type __Thunk__Instrument_AtomicLoad_##Type(Type* Atomic, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder);

	#define INSTRUMENT_STORE_FUNC(Type) \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API void __Thunk__Instrument_AtomicStore_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder);

	#define INSTRUMENT_EXCHANGE_FUNC(Type) \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API Type __Thunk__Instrument_AtomicExchange_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder);

	#define INSTRUMENT_COMPARE_EXCHANGE_FUNC(Type) \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API Type __Thunk__Instrument_AtomicCompareExchange_##Type(Type* Atomic, Type* Expected, Type Val, UE::Instrumentation::FAtomicMemoryOrder SuccessMemoryOrder, UE::Instrumentation::FAtomicMemoryOrder FailureMemoryOrder);

	#define INSTRUMENT_RMW_FUNC(Func, Type) \
	INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_API Type __Thunk__Instrument_Atomic##Func##_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder);

	// Define the functions with C linkage, referenced by the compiler's instrumentation pass.
	INSTRUMENT_LOAD_FUNC(int8)
	INSTRUMENT_LOAD_FUNC(int16)
	INSTRUMENT_LOAD_FUNC(int32)
	INSTRUMENT_LOAD_FUNC(int64)

	INSTRUMENT_STORE_FUNC(int8)
	INSTRUMENT_STORE_FUNC(int16)
	INSTRUMENT_STORE_FUNC(int32)
	INSTRUMENT_STORE_FUNC(int64)

	INSTRUMENT_EXCHANGE_FUNC(int8)
	INSTRUMENT_EXCHANGE_FUNC(int16)
	INSTRUMENT_EXCHANGE_FUNC(int32)
	INSTRUMENT_EXCHANGE_FUNC(int64)

	INSTRUMENT_COMPARE_EXCHANGE_FUNC(int8)
	INSTRUMENT_COMPARE_EXCHANGE_FUNC(int16)
	INSTRUMENT_COMPARE_EXCHANGE_FUNC(int32)
	INSTRUMENT_COMPARE_EXCHANGE_FUNC(int64)

	INSTRUMENT_RMW_FUNC(FetchAdd, int8)
	INSTRUMENT_RMW_FUNC(FetchAdd, int16)
	INSTRUMENT_RMW_FUNC(FetchAdd, int32)
	INSTRUMENT_RMW_FUNC(FetchAdd, int64)
	INSTRUMENT_RMW_FUNC(FetchSub, int8)
	INSTRUMENT_RMW_FUNC(FetchSub, int16)
	INSTRUMENT_RMW_FUNC(FetchSub, int32)
	INSTRUMENT_RMW_FUNC(FetchSub, int64)
	INSTRUMENT_RMW_FUNC(FetchOr, int8)
	INSTRUMENT_RMW_FUNC(FetchOr, int16)
	INSTRUMENT_RMW_FUNC(FetchOr, int32)
	INSTRUMENT_RMW_FUNC(FetchOr, int64)
	INSTRUMENT_RMW_FUNC(FetchXor, int8)
	INSTRUMENT_RMW_FUNC(FetchXor, int16)
	INSTRUMENT_RMW_FUNC(FetchXor, int32)
	INSTRUMENT_RMW_FUNC(FetchXor, int64)
	INSTRUMENT_RMW_FUNC(FetchAnd, int8)
	INSTRUMENT_RMW_FUNC(FetchAnd, int16)
	INSTRUMENT_RMW_FUNC(FetchAnd, int32)
	INSTRUMENT_RMW_FUNC(FetchAnd, int64)

	#undef INSTRUMENT_LOAD_FUNC
	#undef INSTRUMENT_STORE_FUNC
	#undef INSTRUMENT_EXCHANGE_FUNC
	#undef INSTRUMENT_COMPARE_EXCHANGE_FUNC
	#undef INSTRUMENT_RMW_FUNC
}

#endif // USING_INSTRUMENTATION