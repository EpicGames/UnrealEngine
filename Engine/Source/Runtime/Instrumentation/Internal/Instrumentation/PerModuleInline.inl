// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "Instrumentation/Types.h"
#include "Instrumentation/Defines.h"
#include "Instrumentation/EntryPoints.h"

#if defined(USE_INSTRUMENTATION_PERMODULE_HOTPATCH) && USE_INSTRUMENTATION_PERMODULE_HOTPATCH
	#define INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES INSTRUMENTATION_FUNCTION_HOTPATCHABLE	
#else
	#define INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES INSTRUMENTATION_ENTRYPOINT_ATTRIBUTES	
#endif

// Define the functions with C linkage, referenced by the compiler's instrumentation pass.
// We redirect everything to the instrumentation layer thunks so that every module instrumentation ends up at the same place.
extern "C" {

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void AnnotateHappensBefore(const char* f, int l, void* addr)
	{
		__Thunk__AnnotateHappensBefore(f, l, addr);
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void AnnotateHappensAfter(const char* f, int l, void* addr)
	{
		__Thunk__AnnotateHappensAfter(f, l, addr);
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_FuncEntry(void* ReturnAddress)
	{
		__Thunk__Instrument_FuncEntry(ReturnAddress);
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_FuncExit()
	{
		__Thunk__Instrument_FuncExit();
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_VPtr_Store(void** Address, void* Value)
	{
		__Thunk__Instrument_VPtr_Store(Address, Value);
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_VPtr_Load(void** Address)
	{
		__Thunk__Instrument_VPtr_Load(Address);
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_Store(uint64 Address, uint32 Size)
	{
		__Thunk__Instrument_Store(Address, Size);
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_Load(uint64 Address, uint32 Size)
	{
		__Thunk__Instrument_Load(Address, Size);
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_StoreRange(uint64 Address, uint32 Size)
	{
		__Thunk__Instrument_StoreRange(Address, Size);
	}

	INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_LoadRange(uint64 Address, uint32 Size)
	{
		__Thunk__Instrument_LoadRange(Address, Size);
	}

#define INSTRUMENT_LOAD_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES Type __Instrument_AtomicLoad_##Type(Type* Atomic, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return __Thunk__Instrument_AtomicLoad_##Type(Atomic, MemoryOrder); \
}

#define INSTRUMENT_STORE_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES void __Instrument_AtomicStore_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	__Thunk__Instrument_AtomicStore_##Type(Atomic, Val, MemoryOrder); \
}

#define INSTRUMENT_EXCHANGE_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES Type __Instrument_AtomicExchange_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return __Thunk__Instrument_AtomicExchange_##Type(Atomic, Val, MemoryOrder); \
}

#define INSTRUMENT_COMPARE_EXCHANGE_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES Type __Instrument_AtomicCompareExchange_##Type(Type* Atomic, Type* Expected, Type Val, UE::Instrumentation::FAtomicMemoryOrder SuccessMemoryOrder, UE::Instrumentation::FAtomicMemoryOrder FailureMemoryOrder) \
{ \
	return __Thunk__Instrument_AtomicCompareExchange_##Type(Atomic, Expected, Val, SuccessMemoryOrder, FailureMemoryOrder); \
} 

#define INSTRUMENT_FETCHADD_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES Type __Instrument_AtomicFetchAdd_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return __Thunk__Instrument_AtomicFetchAdd_##Type(Atomic, Val, MemoryOrder); \
} 

#define INSTRUMENT_FETCHSUB_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES Type __Instrument_AtomicFetchSub_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return __Thunk__Instrument_AtomicFetchSub_##Type(Atomic, Val, MemoryOrder); \
} 

#define INSTRUMENT_FETCHOR_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES Type __Instrument_AtomicFetchOr_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return __Thunk__Instrument_AtomicFetchOr_##Type(Atomic, Val, MemoryOrder); \
} 

#define INSTRUMENT_FETCHXOR_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES Type __Instrument_AtomicFetchXor_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return __Thunk__Instrument_AtomicFetchXor_##Type(Atomic, Val, MemoryOrder); \
} 

#define INSTRUMENT_FETCHAND_FUNC_IMPL(Type) \
INSTRUMENTATION_PERMODULE_FUNCTION_ATTRIBUTES Type __Instrument_AtomicFetchAnd_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return __Thunk__Instrument_AtomicFetchAnd_##Type(Atomic, Val, MemoryOrder); \
} 

INSTRUMENT_LOAD_FUNC_IMPL(int8)
INSTRUMENT_LOAD_FUNC_IMPL(int16)
INSTRUMENT_LOAD_FUNC_IMPL(int32)
INSTRUMENT_LOAD_FUNC_IMPL(int64)

INSTRUMENT_STORE_FUNC_IMPL(int8)
INSTRUMENT_STORE_FUNC_IMPL(int16)
INSTRUMENT_STORE_FUNC_IMPL(int32)
INSTRUMENT_STORE_FUNC_IMPL(int64)

INSTRUMENT_EXCHANGE_FUNC_IMPL(int8)
INSTRUMENT_EXCHANGE_FUNC_IMPL(int16)
INSTRUMENT_EXCHANGE_FUNC_IMPL(int32)
INSTRUMENT_EXCHANGE_FUNC_IMPL(int64)

INSTRUMENT_COMPARE_EXCHANGE_FUNC_IMPL(int8)
INSTRUMENT_COMPARE_EXCHANGE_FUNC_IMPL(int16)
INSTRUMENT_COMPARE_EXCHANGE_FUNC_IMPL(int32)
INSTRUMENT_COMPARE_EXCHANGE_FUNC_IMPL(int64)

INSTRUMENT_FETCHADD_FUNC_IMPL(int8)
INSTRUMENT_FETCHADD_FUNC_IMPL(int16)
INSTRUMENT_FETCHADD_FUNC_IMPL(int32)
INSTRUMENT_FETCHADD_FUNC_IMPL(int64)
INSTRUMENT_FETCHSUB_FUNC_IMPL(int8)
INSTRUMENT_FETCHSUB_FUNC_IMPL(int16)
INSTRUMENT_FETCHSUB_FUNC_IMPL(int32)
INSTRUMENT_FETCHSUB_FUNC_IMPL(int64)
INSTRUMENT_FETCHOR_FUNC_IMPL(int8)
INSTRUMENT_FETCHOR_FUNC_IMPL(int16)
INSTRUMENT_FETCHOR_FUNC_IMPL(int32)
INSTRUMENT_FETCHOR_FUNC_IMPL(int64)
INSTRUMENT_FETCHXOR_FUNC_IMPL(int8)
INSTRUMENT_FETCHXOR_FUNC_IMPL(int16)
INSTRUMENT_FETCHXOR_FUNC_IMPL(int32)
INSTRUMENT_FETCHXOR_FUNC_IMPL(int64)
INSTRUMENT_FETCHAND_FUNC_IMPL(int8)
INSTRUMENT_FETCHAND_FUNC_IMPL(int16)
INSTRUMENT_FETCHAND_FUNC_IMPL(int32)
INSTRUMENT_FETCHAND_FUNC_IMPL(int64)

#undef INSTRUMENT_LOAD_FUNC_IMPL
#undef INSTRUMENT_STORE_FUNC_IMPL
#undef INSTRUMENT_EXCHANGE_FUNC_IMPL
#undef INSTRUMENT_COMPARE_EXCHANGE_FUNC_IMPL
#undef INSTRUMENT_FETCHADD_FUNC_IMPL
#undef INSTRUMENT_FETCHSUB_FUNC_IMPL
#undef INSTRUMENT_FETCHOR_FUNC_IMPL
#undef INSTRUMENT_FETCHXOR_FUNC_IMPL
#undef INSTRUMENT_FETCHAND_FUNC_IMPL

}

#endif // USING_INSTRUMENTATION