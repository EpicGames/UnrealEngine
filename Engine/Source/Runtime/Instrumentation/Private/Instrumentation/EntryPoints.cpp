// Copyright Epic Games, Inc. All Rights Reserved.

#include "Instrumentation/EntryPoints.h"

#if USING_INSTRUMENTATION

#include "Instrumentation/Defines.h"
#include "Templates/Atomic.h"
#include "CoreTypes.h"

#define INSTRUMENTATION_THUNK_ATTRIBUTES INSTRUMENTATION_FUNCTION_ATTRIBUTES INSTRUMENTATION_FUNCTION_HOTPATCHABLE INSTRUMENTATION_API

// Define the functions with C linkage, referenced by the compiler's instrumentation pass.
extern "C" {

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__AnnotateHappensBefore(const char* f, int l, void* addr)
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__AnnotateHappensAfter(const char* f, int l, void* addr)
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_FuncEntry(void* ReturnAddress)
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_FuncExit()
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_Store(uint64 Address, uint32 Size)
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_Load(uint64 Address, uint32 Size)
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_VPtr_Store(void** Address, void* Value)
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_VPtr_Load(void** Address)
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_StoreRange(uint64 Address, uint32 Size)
{
}

INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_LoadRange(uint64 Address, uint32 Size)
{
}

#define INSTRUMENT_LOAD_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES Type __Thunk__Instrument_AtomicLoad_##Type(Type* Atomic, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return UE::Core::Private::Atomic::Load(Atomic); \
} 

#define INSTRUMENT_STORE_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES void __Thunk__Instrument_AtomicStore_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	UE::Core::Private::Atomic::Store(Atomic, Val); \
} 

#define INSTRUMENT_EXCHANGE_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES Type __Thunk__Instrument_AtomicExchange_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return UE::Core::Private::Atomic::Exchange(Atomic, Val); \
} 

#define INSTRUMENT_COMPARE_EXCHANGE_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES Type __Thunk__Instrument_AtomicCompareExchange_##Type(Type* Atomic, Type* Expected, Type Val, UE::Instrumentation::FAtomicMemoryOrder SuccessMemoryOrder, UE::Instrumentation::FAtomicMemoryOrder FailureMemoryOrder) \
{ \
	Type Ret = UE::Core::Private::Atomic::CompareExchange(Atomic, *Expected, Val); \
	if (Ret != *Expected) \
	{ \
		*Expected = Ret; \
	} \
	return Ret; \
} 

#define INSTRUMENT_FETCHADD_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES Type __Thunk__Instrument_AtomicFetchAdd_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return UE::Core::Private::Atomic::AddExchange(Atomic, Val); \
} 

#define INSTRUMENT_FETCHSUB_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES Type __Thunk__Instrument_AtomicFetchSub_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return UE::Core::Private::Atomic::SubExchange(Atomic, Val); \
} 

#define INSTRUMENT_FETCHOR_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES Type __Thunk__Instrument_AtomicFetchOr_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return UE::Core::Private::Atomic::OrExchange(Atomic, Val); \
} 

#define INSTRUMENT_FETCHXOR_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES Type __Thunk__Instrument_AtomicFetchXor_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return UE::Core::Private::Atomic::XorExchange(Atomic, Val); \
} 

#define INSTRUMENT_FETCHAND_FUNC_THUNK(Type) \
INSTRUMENTATION_THUNK_ATTRIBUTES Type __Thunk__Instrument_AtomicFetchAnd_##Type(Type* Atomic, Type Val, UE::Instrumentation::FAtomicMemoryOrder MemoryOrder) \
{ \
	return UE::Core::Private::Atomic::AndExchange(Atomic, Val); \
} 

INSTRUMENT_LOAD_FUNC_THUNK(int8)
INSTRUMENT_LOAD_FUNC_THUNK(int16)
INSTRUMENT_LOAD_FUNC_THUNK(int32)
INSTRUMENT_LOAD_FUNC_THUNK(int64)

INSTRUMENT_STORE_FUNC_THUNK(int8)
INSTRUMENT_STORE_FUNC_THUNK(int16)
INSTRUMENT_STORE_FUNC_THUNK(int32)
INSTRUMENT_STORE_FUNC_THUNK(int64)

INSTRUMENT_EXCHANGE_FUNC_THUNK(int8)
INSTRUMENT_EXCHANGE_FUNC_THUNK(int16)
INSTRUMENT_EXCHANGE_FUNC_THUNK(int32)
INSTRUMENT_EXCHANGE_FUNC_THUNK(int64)

INSTRUMENT_COMPARE_EXCHANGE_FUNC_THUNK(int8)
INSTRUMENT_COMPARE_EXCHANGE_FUNC_THUNK(int16)
INSTRUMENT_COMPARE_EXCHANGE_FUNC_THUNK(int32)
INSTRUMENT_COMPARE_EXCHANGE_FUNC_THUNK(int64)

INSTRUMENT_FETCHADD_FUNC_THUNK(int8)
INSTRUMENT_FETCHADD_FUNC_THUNK(int16)
INSTRUMENT_FETCHADD_FUNC_THUNK(int32)
INSTRUMENT_FETCHADD_FUNC_THUNK(int64)
INSTRUMENT_FETCHSUB_FUNC_THUNK(int8)
INSTRUMENT_FETCHSUB_FUNC_THUNK(int16)
INSTRUMENT_FETCHSUB_FUNC_THUNK(int32)
INSTRUMENT_FETCHSUB_FUNC_THUNK(int64)
INSTRUMENT_FETCHOR_FUNC_THUNK(int8)
INSTRUMENT_FETCHOR_FUNC_THUNK(int16)
INSTRUMENT_FETCHOR_FUNC_THUNK(int32)
INSTRUMENT_FETCHOR_FUNC_THUNK(int64)
INSTRUMENT_FETCHXOR_FUNC_THUNK(int8)
INSTRUMENT_FETCHXOR_FUNC_THUNK(int16)
INSTRUMENT_FETCHXOR_FUNC_THUNK(int32)
INSTRUMENT_FETCHXOR_FUNC_THUNK(int64)
INSTRUMENT_FETCHAND_FUNC_THUNK(int8)
INSTRUMENT_FETCHAND_FUNC_THUNK(int16)
INSTRUMENT_FETCHAND_FUNC_THUNK(int32)
INSTRUMENT_FETCHAND_FUNC_THUNK(int64)

}

#endif // USING_INSTRUMENTATION