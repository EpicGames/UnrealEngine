// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if USING_INSTRUMENTATION

#include "Instrumentation/Defines.h"

#include "CoreTypes.h"

#include <atomic>

namespace UE::Instrumentation
{

	// ------------------------------------------------------------------------------
	// Atomics.
	// ------------------------------------------------------------------------------
	enum FAtomicMemoryOrder : int8 {
		MEMORY_ORDER_RELAXED,
		MEMORY_ORDER_CONSUME,
		MEMORY_ORDER_ACQUIRE,
		MEMORY_ORDER_RELEASE,
		MEMORY_ORDER_ACQ_REL,
		MEMORY_ORDER_SEQ_CST
	};

	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline const TCHAR* LexToString(FAtomicMemoryOrder Order)
	{
		switch (Order)
		{
		case MEMORY_ORDER_RELAXED:
			return TEXT("relaxed");
		case MEMORY_ORDER_CONSUME:
			return TEXT("consume");
		case MEMORY_ORDER_ACQUIRE:
			return TEXT("acquire");
		case MEMORY_ORDER_RELEASE:
			return TEXT("release");
		case MEMORY_ORDER_ACQ_REL:
			return TEXT("acq_rel");
		case MEMORY_ORDER_SEQ_CST:
			return TEXT("seq_cst");
		default:
			return TEXT("unknown");
		}
	};

	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline std::memory_order ToStdMemoryOrder(FAtomicMemoryOrder MemoryOrder)
	{
		switch (MemoryOrder)
		{
		case MEMORY_ORDER_RELAXED:
			return std::memory_order_relaxed;
		case MEMORY_ORDER_CONSUME:
			return std::memory_order_consume;
		case MEMORY_ORDER_ACQUIRE:
			return std::memory_order_acquire;
		case MEMORY_ORDER_RELEASE:
			return std::memory_order_release;
		case MEMORY_ORDER_ACQ_REL:
			return std::memory_order_acq_rel;
		default:
			return std::memory_order_seq_cst;
		}
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline bool IsAtomicOrderAcquire(FAtomicMemoryOrder Order)
	{
		return Order == FAtomicMemoryOrder::MEMORY_ORDER_ACQ_REL || Order == FAtomicMemoryOrder::MEMORY_ORDER_ACQUIRE
			|| Order == FAtomicMemoryOrder::MEMORY_ORDER_CONSUME || Order == FAtomicMemoryOrder::MEMORY_ORDER_SEQ_CST;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline bool IsAtomicOrderRelease(FAtomicMemoryOrder Order)
	{
		return Order == FAtomicMemoryOrder::MEMORY_ORDER_ACQ_REL || Order == FAtomicMemoryOrder::MEMORY_ORDER_RELEASE
			|| Order == FAtomicMemoryOrder::MEMORY_ORDER_SEQ_CST;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES inline bool IsAtomicOrderAcquireRelease(FAtomicMemoryOrder Order)
	{
		return Order == FAtomicMemoryOrder::MEMORY_ORDER_ACQ_REL || Order == FAtomicMemoryOrder::MEMORY_ORDER_SEQ_CST;
	}

	INSTRUMENTATION_FUNCTION_ATTRIBUTES FORCEINLINE bool IsAtomicOrderRelaxed(FAtomicMemoryOrder Order)
	{
		return Order == FAtomicMemoryOrder::MEMORY_ORDER_RELAXED;
	}

} // UE::Instrumentation

#endif // USING_INSTRUMENTATION